{
	"FileVersion": 3,
	"Version": 1.0,
	"VersionName": "1.0",
	"FriendlyName": "Pixel Streaming Player",
	"Description": "Support for receiving a pixel streaming stream and displaying it in game.",
	"Category": "Graphics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Platforms/PixelStreaming/index.html",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PixelStreamingPlayer",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreamingPlayerEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "PixelStreaming",
			"Enabled": true
		}
	]
}
