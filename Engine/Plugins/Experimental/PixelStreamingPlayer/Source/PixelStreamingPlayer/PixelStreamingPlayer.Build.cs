// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class PixelStreamingPlayer : ModuleRules
	{
		public PixelStreamingPlayer(ReadOnlyTargetRules Target) : base(Target)
		{
			var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

			PublicIncludePaths.AddRange(
				new string[] {
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"WebRTC",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"MediaAssets",
					"PixelStreaming",
					"RenderCore",
					"RHI",
					"WebSockets",
				});
		}
	}
}
