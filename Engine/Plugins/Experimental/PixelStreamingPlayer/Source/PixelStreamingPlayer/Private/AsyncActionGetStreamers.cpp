// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncActionGetStreamers.h"

void UAsyncAction_GetStreamers::Activate()
{
	SignallingConnectionComponent->AsyncRequestStreamerList([this](const TArray<FString>& StreamerList) {
		Completed.Broadcast(StreamerList);
		SetReadyToDestroy();
	});
}

UAsyncAction_GetStreamers* UAsyncAction_GetStreamers::GetStreamerIdList(UPixelStreamingSignallingComponent* SignallingComponent)
{
	UAsyncAction_GetStreamers* Action = NewObject<UAsyncAction_GetStreamers>();
	Action->SignallingConnectionComponent = SignallingComponent;
	Action->RegisterWithGameInstance(SignallingComponent);
	return Action;
}
