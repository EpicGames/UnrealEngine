// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingSignallingComponent.h"
#include "AsyncActionGetStreamers.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnStreamerListCompleted, const TArray<FString>&, StreamerList);

UCLASS()
class PIXELSTREAMINGPLAYER_API UAsyncAction_GetStreamers : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	virtual void Activate() override;

	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true", Category = "PixelStreaming"))
	static UAsyncAction_GetStreamers* GetStreamerIdList(UPixelStreamingSignallingComponent* SignallingComponent);

	UPROPERTY(BlueprintAssignable)
	FOnStreamerListCompleted Completed;

	UPixelStreamingSignallingComponent* SignallingConnectionComponent;
};
