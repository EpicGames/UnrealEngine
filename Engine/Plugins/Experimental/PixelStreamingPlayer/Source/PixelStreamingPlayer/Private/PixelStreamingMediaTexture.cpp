// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingMediaTexture.h"
#include "PixelStreamingMediaTextureResource.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "Async/Async.h"
#include "WebRTCIncludes.h"

constexpr int32 DEFAULT_WIDTH = 1920;
constexpr int32 DEFAULT_HEIGHT = 1080;

UPixelStreamingMediaTexture::UPixelStreamingMediaTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetResource(nullptr);
}

void UPixelStreamingMediaTexture::BeginDestroy()
{
	AsyncTask(ENamedThreads::ActualRenderingThread, [this]() {
		FScopeLock Lock(&RenderSyncContext);
		RenderTarget = nullptr;
	});

	Super::BeginDestroy();
}

void UPixelStreamingMediaTexture::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (CurrentResource != nullptr)
	{
		CumulativeResourceSize.AddUnknownMemoryBytes(CurrentResource->GetResourceSize());
	}
}

FTextureResource* UPixelStreamingMediaTexture::CreateResource()
{
	if (CurrentResource != nullptr)
	{
		SetResource(nullptr);
		delete CurrentResource;
	}

	CurrentResource = new FPixelStreamingMediaTextureResource(this);
	InitializeResources();

	return CurrentResource;
}

void UPixelStreamingMediaTexture::OnFrame(FTextureRHIRef Frame)
{
	AsyncTask(ENamedThreads::ActualRenderingThread, [this, Frame]() {
		FScopeLock Lock(&RenderSyncContext);

		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		UpdateTextureReference(RHICmdList, Frame);
	});
}

void UPixelStreamingMediaTexture::InitializeResources()
{
	ENQUEUE_RENDER_COMMAND(FPixelStreamingMediaTextureUpdateTextureReference)
	([this](FRHICommandListImmediate& RHICmdList) {
		// Set the default video texture to reference nothing
		FTextureRHIRef ShaderTexture2D;
		FTextureRHIRef RenderableTexture;
	
		FRHITextureCreateDesc RenderTargetTextureDesc =
			FRHITextureCreateDesc::Create2D(TEXT(""), DEFAULT_WIDTH, DEFAULT_HEIGHT, EPixelFormat::PF_B8G8R8A8)
				.SetClearValue(FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f)))
				.SetFlags(ETextureCreateFlags::ShaderResource | TexCreate_RenderTargetable)
				.SetInitialState(ERHIAccess::SRVMask);
	
		RenderableTexture = RHICmdList.CreateTexture(RenderTargetTextureDesc);
		ShaderTexture2D = RenderableTexture;
	
		CurrentResource->TextureRHI = RenderableTexture;

		RHICmdList.UpdateTextureReference(TextureReference.TextureReferenceRHI, CurrentResource->TextureRHI);
	});
}

void UPixelStreamingMediaTexture::UpdateTextureReference(FRHICommandList& RHICmdList, FTextureRHIRef Reference)
{
	if (CurrentResource != nullptr)
	{
		if (Reference.IsValid() && CurrentResource->TextureRHI != Reference)
		{
			CurrentResource->TextureRHI = Reference;
			RHICmdList.UpdateTextureReference(TextureReference.TextureReferenceRHI, CurrentResource->TextureRHI);
		}
		else if (!Reference.IsValid())
		{
			if (CurrentResource != nullptr)
			{
				InitializeResources();

				// Make sure RenderThread is executed before continuing
				FlushRenderingCommands();
			}
		}
	}
}
