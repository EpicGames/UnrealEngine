// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingMediaTextureResource.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "RenderUtils.h"

FPixelStreamingMediaTextureResource::FPixelStreamingMediaTextureResource(UPixelStreamingMediaTexture* Owner)
{
	MediaTexture = Owner;
}

void FPixelStreamingMediaTextureResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (MediaTexture != nullptr)
	{
		FSamplerStateInitializerRHI SamplerStateInitializer(
			(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(MediaTexture),
			AM_Border, AM_Border, AM_Wrap);
		SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
	}
}

void FPixelStreamingMediaTextureResource::ReleaseRHI()
{
	TextureRHI.SafeRelease();

	if (MediaTexture != nullptr)
	{
		RHIClearTextureReference(MediaTexture->TextureReference.TextureReferenceRHI);
	}
}

uint32 FPixelStreamingMediaTextureResource::GetSizeX() const
{
	return TextureRHI.IsValid() ? TextureRHI->GetSizeXYZ().X : 0;
}

uint32 FPixelStreamingMediaTextureResource::GetSizeY() const
{
	return TextureRHI.IsValid() ? TextureRHI->GetSizeXYZ().Y : 0;
}

SIZE_T FPixelStreamingMediaTextureResource::GetResourceSize()
{
	return CalcTextureSize(GetSizeX(), GetSizeY(), EPixelFormat::PF_A8R8G8B8, 1);
}
