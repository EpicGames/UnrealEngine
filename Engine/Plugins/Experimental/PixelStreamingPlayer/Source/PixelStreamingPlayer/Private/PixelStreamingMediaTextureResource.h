// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TextureResource.h"
#include "PixelStreamingMediaTexture.h"

/**
 * The actual texture resource for a FPixelStreamingMediaTexture. Contains the RHI Texture and
 * Sampler information.
 */
class PIXELSTREAMINGPLAYER_API FPixelStreamingMediaTextureResource : public FTextureResource
{
public:
	FPixelStreamingMediaTextureResource(UPixelStreamingMediaTexture* Owner);
	virtual ~FPixelStreamingMediaTextureResource() override { TextureRHI.SafeRelease(); }

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	virtual uint32 GetSizeX() const override;
	virtual uint32 GetSizeY() const override;

	SIZE_T GetResourceSize();

private:
	UPixelStreamingMediaTexture* MediaTexture = nullptr;
};
