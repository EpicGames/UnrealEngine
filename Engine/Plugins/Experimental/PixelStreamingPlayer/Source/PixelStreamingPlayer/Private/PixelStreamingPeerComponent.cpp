// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingPeerComponent.h"
#include "PixelStreamingPlayerPrivate.h"
#include "PixelStreamingPeerConnection.h"
// #include "RTCStatsCollector.h"
#include "Async/Async.h"

UPixelStreamingPeerComponent::UPixelStreamingPeerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPixelStreamingPeerComponent::SetConfig(const FPixelStreamingRTCConfigWrapper& Config)
{
	PeerConnection = FPixelStreamingPeerConnection::Create(Config.Config);
	// TODO (william.belcher): Move the RTCStatsCollector from PixelStreaming to an Internal folder so it can be accessed across modules.
	// #if WEBRTC_5414
	// 	PeerConnection->SetWebRTCStatsCallback(rtc::scoped_refptr<UE::PixelStreaming::FRTCStatsCollector>(new UE::PixelStreaming::FRTCStatsCollector(TEXT("Streamer"))));
	// #else
	// 	PeerConnection->SetWebRTCStatsCallback(new rtc::RefCountedObject<UE::PixelStreaming::FRTCStatsCollector>(TEXT("Streamer")));
	// #endif

	if (PeerConnection)
	{
		PeerConnection->SetVideoSink(VideoSink);
		PeerConnection->OnEmitIceCandidate.AddLambda([this](const webrtc::IceCandidateInterface* Candidate) {
			FPixelStreamingIceCandidateWrapper CandidateWrapper(*Candidate);
			AsyncTask(ENamedThreads::GameThread, [this, CandidateWrapper]() {
				OnIceCandidate.Broadcast(CandidateWrapper);
			});
		});
		PeerConnection->OnIceStateChanged.AddLambda([this](webrtc::PeerConnectionInterface::IceConnectionState NewState) {
			AsyncTask(ENamedThreads::GameThread, [this, NewState]() {
				OnIceConnectionChange(NewState);
			});
		});
	}
	else
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "SetConfig Failed.");
	}
}

FPixelStreamingSessionDescriptionWrapper UPixelStreamingPeerComponent::CreateOffer()
{
	if (PeerConnection)
	{
		FPixelStreamingSessionDescriptionWrapper Wrapper;
		FEvent*									 TaskEvent = FPlatformProcess::GetSynchEventFromPool();
		const auto								 OnGeneralFailure = [&TaskEvent](const FString& ErrorMsg) {
			  UE_LOGF(LogPixelStreamingPlayer, Error, "CreateOffer Failed: %ls", *ErrorMsg);
			  TaskEvent->Trigger();
		};
		AsyncTask(ENamedThreads::AnyNormalThreadNormalTask, [this, &TaskEvent, &Wrapper, &OnGeneralFailure]() {
			PeerConnection->CreateOffer(
				FPixelStreamingPeerConnection::EReceiveMediaOption::All,
				[&TaskEvent, &Wrapper](const webrtc::SessionDescriptionInterface* Sdp) {
					// success
					Wrapper.SDP = Sdp;
					TaskEvent->Trigger();
				},
				OnGeneralFailure);
		});
		TaskEvent->Wait();
		FPlatformProcess::ReturnSynchEventToPool(TaskEvent);

		if (Wrapper.SDP == nullptr)
		{
			UE_LOGF(LogPixelStreamingPlayer, Error, "CreateOffer Failed: Timeout");
		}

		return Wrapper;
	}
	else
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "Failed to create offer. Call SetConfig first.");
		return {};
	}
}

FPixelStreamingSessionDescriptionWrapper UPixelStreamingPeerComponent::CreateAnswer(const FString& Sdp)
{
	if (PeerConnection)
	{
		FPixelStreamingSessionDescriptionWrapper Wrapper;
		FEvent*									 TaskEvent = FPlatformProcess::GetSynchEventFromPool();
		const auto								 OnGeneralFailure = [&TaskEvent](const FString& ErrorMsg) {
			  UE_LOGF(LogPixelStreamingPlayer, Error, "CreateAnswer Failed: %ls", *ErrorMsg);
			  TaskEvent->Trigger();
		};
		AsyncTask(ENamedThreads::AnyNormalThreadNormalTask, [this, &TaskEvent, &Sdp, &Wrapper, &OnGeneralFailure]() {
			PeerConnection->ReceiveOffer(
				Sdp,
				[this, &TaskEvent, &Wrapper, &OnGeneralFailure]() {
					// success
					PeerConnection->CreateAnswer(
						FPixelStreamingPeerConnection::EReceiveMediaOption::All,
						[&TaskEvent, &Wrapper](const webrtc::SessionDescriptionInterface* Sdp) {
							// success
							Wrapper.SDP = Sdp;
							TaskEvent->Trigger();
						},
						OnGeneralFailure);
				},
				OnGeneralFailure);
		});
		TaskEvent->Wait();
		FPlatformProcess::ReturnSynchEventToPool(TaskEvent);

		if (Wrapper.SDP == nullptr)
		{
			UE_LOGF(LogPixelStreamingPlayer, Error, "CreateAnswer Failed: Timeout");
		}

		return Wrapper;
	}
	else
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "Failed to create answer. Call SetConfig first.");
		return {};
	}
}

void UPixelStreamingPeerComponent::ReceiveAnswer(const FString& Answer)
{
	if (PeerConnection)
	{
		FEvent*	   TaskEvent = FPlatformProcess::GetSynchEventFromPool();
		const auto OnGeneralFailure = [&TaskEvent](const FString& ErrorMsg) {
			UE_LOGF(LogPixelStreamingPlayer, Error, "ReceiveAnswer Failed: %ls", *ErrorMsg);
			TaskEvent->Trigger();
		};
		AsyncTask(ENamedThreads::AnyNormalThreadNormalTask, [this, &TaskEvent, &Answer, &OnGeneralFailure]() {
			PeerConnection->ReceiveAnswer(
				Answer,
				[this, &TaskEvent, &OnGeneralFailure]() {
					// success
					TaskEvent->Trigger();
				},
				OnGeneralFailure);
		});
		TaskEvent->Wait();
		FPlatformProcess::ReturnSynchEventToPool(TaskEvent);
	}
	else
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "Failed to receive answer. Call SetConfig first.");
	}
}

void UPixelStreamingPeerComponent::ReceiveIceCandidate(const FPixelStreamingIceCandidateWrapper& Candidate)
{
	if (PeerConnection)
	{
		PeerConnection->AddRemoteIceCandidate(Candidate.Mid, Candidate.MLineIndex, Candidate.SDP);
	}
}

void UPixelStreamingPeerComponent::OnIceConnectionChange(webrtc::PeerConnectionInterface::IceConnectionState NewState)
{
	if (NewState == webrtc::PeerConnectionInterface::IceConnectionState::kIceConnectionConnected)
	{
		OnIceConnection.Broadcast(0);
	}
	else if (NewState == webrtc::PeerConnectionInterface::IceConnectionState::kIceConnectionDisconnected)
	{
		OnIceDisconnection.Broadcast();
	}
}
