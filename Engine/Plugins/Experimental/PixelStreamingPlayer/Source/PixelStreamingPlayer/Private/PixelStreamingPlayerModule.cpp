// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingPlayerPrivate.h"
#include "Modules/ModuleManager.h"
#include "WebSocketsModule.h"
#include "WebRTCIncludes.h" // IWYU pragma: keep
#include "Async/Async.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY(LogPixelStreamingPlayer);

class FPixelStreamingPlayerModule : public IModuleInterface
{
public:
private:
	/** IModuleInterface implementation */
	void StartupModule() override
	{
		FModuleManager::LoadModuleChecked<FWebSocketsModule>("WebSockets");
		rtc::InitializeSSL();

		// Hack (Luke) - If we have the PS player plugin enabled it will try to set the negotiate codecs to true because it needs it.
		// Proper solution, always negotiate codecs (need to ensure SFU can handle this).
		AsyncTask(ENamedThreads::GameThread, []() {
			IConsoleVariable* CVarNegotiateCodecs = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming.WebRTC.NegotiateCodecs"));
			if (CVarNegotiateCodecs)
			{
				CVarNegotiateCodecs->Set(true);
			}
		});
	}
	void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FPixelStreamingPlayerModule, PixelStreamingPlayer)
