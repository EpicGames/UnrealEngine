// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingSignallingComponent.h"
#include "PixelStreamingSignallingConnection.h"
#include "WebSocketsModule.h"
#include "PixelStreamingPlayerPrivate.h"
#include "StreamMediaSource.h"
#include <media/base/media_config.h>
#include <vector>

UPixelStreamingSignallingComponent::UPixelStreamingSignallingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SignallingServerConnectionObserver = MakeShared<FSignallingObserver>(this);
	SignallingConnection = MakeUnique<FPixelStreamingSignallingConnection>(SignallingServerConnectionObserver);
	// I dont disable keep alive here because it causes a deadlock.
	// This is because the game thread is waiting to load this component but the timer must be set
	// on the game thread.
}

void UPixelStreamingSignallingComponent::Connect(const FString& Url)
{
	SignallingConnection->SetKeepAlive(false);
	SignallingConnection->SetAutoReconnect(true);
	FString URL = (MediaSource == nullptr ? Url : MediaSource->GetUrl());

	TArray<FString> Components;
	URL.ParseIntoArray(Components, TEXT(":"), true);

	if(Components.Num() < 2)
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "Incorrectly formated connection URL. Ensure URL is in the format (protocol)://(ip):(port)");
		return;
	}

	if(Components.Num() == 2)
	{
		// The user hasn't specified the port. Append port 80 as Pixel Streaming defaults to 8888
		URL += TEXT(":80");
	}
	
	SignallingConnection->TryConnect(URL);
}

void UPixelStreamingSignallingComponent::Disconnect()
{
	SignallingConnection->Disconnect(TEXT("Pixel Streaming player plugin disconnected"));
}

void UPixelStreamingSignallingComponent::Subscribe(const FString& StreamerId)
{
	SignallingConnection->SendSubscribe(StreamerId);
}

void UPixelStreamingSignallingComponent::Unsubscribe()
{
	SignallingConnection->SendUnsubscribe();
}

void UPixelStreamingSignallingComponent::SendOffer(const FPixelStreamingSessionDescriptionWrapper& Offer)
{
	if (Offer.SDP == nullptr)
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "Send Offer failed: Offer was null");
	}
	else
	{
		SignallingConnection->SendOffer(*Offer.SDP);
	}
}

void UPixelStreamingSignallingComponent::SendAnswer(const FPixelStreamingSessionDescriptionWrapper& Answer)
{
	if (Answer.SDP == nullptr)
	{
		UE_LOGF(LogPixelStreamingPlayer, Error, "Send Answer failed: Answer was null");
	}
	else
	{
		SignallingConnection->SendAnswer(*Answer.SDP);
	}
}

void UPixelStreamingSignallingComponent::SendIceCandidate(const FPixelStreamingIceCandidateWrapper& CandidateWrapper)
{
	SignallingConnection->SendIceCandidate(*CandidateWrapper.ToWebRTC());
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingConnected()
{
	Parent->OnConnected.Broadcast();
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean)
{
	Parent->OnDisconnected.Broadcast(StatusCode, Reason, bWasClean);
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingError(const FString& ErrorMsg)
{
	Parent->OnConnectionError.Broadcast(ErrorMsg);
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config)
{
	FPixelStreamingRTCConfigWrapper Wrapper;
	Wrapper.Config = Config;
	Parent->OnConfig.Broadcast(Wrapper);
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp)
{
	if (Type == webrtc::SdpType::kOffer)
	{
		Parent->OnOffer.Broadcast(Sdp);
	}
	else if (Type == webrtc::SdpType::kAnswer)
	{
		Parent->OnAnswer.Broadcast(Sdp);
	}
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
{
	FPixelStreamingIceCandidateWrapper CandidateWrapper(SdpMid, SdpMLineIndex, Sdp);
	Parent->OnIceCandidate.Broadcast(CandidateWrapper);
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId)
{
	// TODO (Matthew.Cotton): What to do with data channels?
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingPlayerCount(uint32 Count)
{
	// no-op player count is not interesting to us here
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer)
{
	// no-op our player receiving other players is possible but not a case we are interested in
	UE_LOGF(LogPixelStreamingPlayer, Warning, "Warning: Received message about player connected from SS, this very likely means you have misconfigured the Pixel Streaming player connection and entered the streamer port. Please check your connect node and ensure it using the player port, e.g. 80 (or no port).");
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId)
{
	// no-op our player knowing when other players disconnect is not interesting for now
	UE_LOGF(LogPixelStreamingPlayer, Warning, "Warning: Received message about player disconnected from SS, this very likely means you have misconfigured the Pixel Streaming player connection and entered the streamer port. Please check your connect node and ensure it using the player port, e.g. 80 (or no port).");
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId)
{
	// TODO (Matthew.Cotton): What to do with data channels?
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnSignallingStreamerList(const TArray<FString>& StreamerList)
{
	Parent->OnStreamerListCallback(StreamerList);
}

void UPixelStreamingSignallingComponent::FSignallingObserver::OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate)
{
}

void UPixelStreamingSignallingComponent::AsyncRequestStreamerList(const TFunction<void(const TArray<FString>&)>& Callback)
{
	OnStreamerListCallback = Callback;
	SignallingConnection->RequestStreamerList();
}
