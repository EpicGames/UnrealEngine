// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS

	#include "Windows/AllowWindowsPlatformTypes.h"

	// C4582: constructor is not implicitly called in "api/rtcerror.h", treated as an error by UnrealEngine
	// C6319: Use of the comma-operator in a tested expression causes the left argument to be ignored when it has no side-effects.
	// C6323: Use of arithmetic operator on Boolean type(s).
	#pragma warning(push)
	#pragma warning(disable : 6319 6323)
#endif // PLATFORM_WINDOWS

#include "rtc_base/ssl_adapter.h"
#include "modules/video_capture/video_capture_defines.h"

#if PLATFORM_WINDOWS
	#pragma warning(pop)

	#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_END

#include "PixelStreamingPlayerPrivate.h"