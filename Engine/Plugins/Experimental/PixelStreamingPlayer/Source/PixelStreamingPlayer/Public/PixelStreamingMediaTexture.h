// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2DDynamic.h"
#include "RenderTargetPool.h"
#include "PixelStreamingVideoSink.h"
#include "PixelStreamingMediaTexture.generated.h"

class FPixelStreamingMediaTextureResource;

/**
 * A Texture Object that can be used in materials etc. that takes updates from webrtc frames.
 */
UCLASS(NotBlueprintType, NotBlueprintable, HideDropdown, HideCategories = (ImportSettings, Compression, Texture, Adjustments, Compositing, LevelOfDetail, Object), META = (DisplayName = "PixelStreaming Media Texture"))
class PIXELSTREAMINGPLAYER_API UPixelStreamingMediaTexture : public UTexture2DDynamic, public FPixelStreamingVideoSink
{
	GENERATED_UCLASS_BODY()

protected:
	// UObject overrides.
	virtual void BeginDestroy() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	// UTexture implementation
	virtual FTextureResource* CreateResource() override;

	// FPixelStreamingVideoSink implementation
	virtual void OnFrame(FTextureRHIRef Frame) override;

private:
	void InitializeResources();

	// updates the internal texture resource after each frame.
	void UpdateTextureReference(FRHICommandList& RHICmdList, FTextureRHIRef Reference);

	TArray<uint8_t> Buffer;

	FCriticalSection RenderSyncContext;
	FTextureRHIRef SourceTexture;
	FPooledRenderTargetDesc RenderTargetDescriptor;
	TRefCountPtr<IPooledRenderTarget> RenderTarget;
	FPixelStreamingMediaTextureResource* CurrentResource;
};
