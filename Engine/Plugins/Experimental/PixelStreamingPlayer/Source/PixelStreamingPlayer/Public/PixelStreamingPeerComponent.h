// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingPeerConnection.h"
#include "PixelStreamingSignallingComponent.h"
#include "PixelStreamingMediaTexture.h"
#include "PixelStreamingWebRTCWrappers.h"
#include "PixelStreamingPeerComponent.generated.h"

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingOnIceCandidate, UPixelStreamingPeerComponent, OnIceCandidate, FPixelStreamingIceCandidateWrapper, Candidate);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingOnIceConnection, UPixelStreamingPeerComponent, OnIceConnection, int, Number);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(FPixelStreamingOnIceDisconnection, UPixelStreamingPeerComponent, OnIceDisconnection);

/**
 * A blueprint representation of a Pixel Streaming Peer Connection. Should communicate with a Pixel Streaming Signalling Connection
 * and will accept video sinks to receive video data.
 */
UCLASS(BlueprintType, Blueprintable, Category = "PixelStreaming", META = (DisplayName = "PixelStreaming Peer Component", BlueprintSpawnableComponent))
class PIXELSTREAMINGPLAYER_API UPixelStreamingPeerComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Sets the RTC Configuration for this Peer Connection.
	 * @param Config The RTC configuration for this Peer Connection. Obtained from the signalling server On Config event.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void SetConfig(const FPixelStreamingRTCConfigWrapper& Config);

	/**
	 * Creates an offer.
	 * @return The offer object generated. Send this to the signalling server to initiate negotiation.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	FPixelStreamingSessionDescriptionWrapper CreateOffer();

	/**
	 * Creates an answer to the given offer objet that was provided.
	 * @param Offer The offer SDP string to create an answer for. Should be obtained from the signalling server On Offer event.
	 * @return The answer object generated. Send this to the signalling server to complete negotiation.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	FPixelStreamingSessionDescriptionWrapper CreateAnswer(const FString& Offer);

	/**
	 * Receives an answer from a streamer after we've sent an offer to receive.
	 * @param Offer The answer SDP. Should be obtained from the signalling server On Answer event.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void ReceiveAnswer(const FString& Offer);

	/**
	 * Notify the peer connection of an ICE candidate sent by the singalling connection.
	 * @param Candidate Provided by the singalling connection.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void ReceiveIceCandidate(const FPixelStreamingIceCandidateWrapper& Candidate);

	/**
	 * Once negotiation is completed the Peer Connection can generate Ice Candidate objects. These need to be sent to a signalling server to allow proper connection.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingOnIceCandidate OnIceCandidate;

	/**
	 * Once a connection has been connected and streaming should be available.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingOnIceConnection OnIceConnection;

	/**
	 * When an ice connection is lost.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingOnIceDisconnection OnIceDisconnection;

	/**
	 * A sink for the video data received once this connection has finished negotiating.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Properties", META = (DisplayName = "Pixel Streaming Video Sink", AllowPrivateAccess = true))
	TObjectPtr<UPixelStreamingMediaTexture> VideoSink = nullptr;

private:
	TUniquePtr<FPixelStreamingPeerConnection> PeerConnection;

	void OnIceConnectionChange(webrtc::PeerConnectionInterface::IceConnectionState);
};
