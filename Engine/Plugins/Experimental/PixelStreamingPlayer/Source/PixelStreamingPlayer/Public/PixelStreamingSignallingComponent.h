// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingSignallingConnection.h"
#include "StreamMediaSource.h"
#include "Components/ActorComponent.h"
#include "PixelStreamingWebRTCWrappers.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "PixelStreamingSignallingComponent.generated.h"

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(FPixelStreamingSignallingComponentConnected, UPixelStreamingSignallingComponent, OnConnected);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingSignallingComponentConnectionError, UPixelStreamingSignallingComponent, OnConnectionError, const FString&, ErrorMsg);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_ThreeParams(FPixelStreamingSignallingComponentDisconnected, UPixelStreamingSignallingComponent, OnDisconnected, int32, StatusCode, const FString&, Reason, bool, bWasClean);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingSignallingComponentConfig, UPixelStreamingSignallingComponent, OnConfig, FPixelStreamingRTCConfigWrapper, Config);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingSignallingComponentOffer, UPixelStreamingSignallingComponent, OnOffer, const FString&, Offer);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingSignallingComponentAnswer, UPixelStreamingSignallingComponent, OnAnswer, const FString&, Answer);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingSignallingComponentIceCandidate, UPixelStreamingSignallingComponent, OnIceCandidate, FPixelStreamingIceCandidateWrapper, Candidate);
DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FPixelStreamingSignallingComponentDataChannels, UPixelStreamingSignallingComponent, OnDataChannels, int32, SendStreamId, int32, RecvStreamId);

/**
 * A blueprint class representing a Pixel Streaming Signalling connection. Used to communicate with the signalling server and
 * should route information to the peer connection.
 */
UCLASS(BlueprintType, Blueprintable, Category = "PixelStreaming", META = (DisplayName = "PixelStreaming Signalling Component", BlueprintSpawnableComponent))
class PIXELSTREAMINGPLAYER_API UPixelStreamingSignallingComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Attempt to connect to a specified signalling server.
	 * @param Url The url of the signalling server. Ignored if this component has a MediaSource. In that case the URL on the media source will be used instead.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void Connect(const FString& Url);

	/**
	 * Disconnect from the signalling server. No action if no connection exists.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void Disconnect();

	/**
	 * Send a subscribe message to the signalling server to subscribe to a specific streamer.
	 * @param StreamerId The name of the streamer to subscribe to.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void Subscribe(const FString& StreamerId);

	/**
	 * Send an unsubscribe message to the signalling server to unsubscribe from a streamer.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void Unsubscribe();

	/**
	 * Send an offer created from a Peer Connection to the signalling server.
	 * @param Offer The answer object created from calling CreateAnswer on a Peer Connection.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void SendOffer(const FPixelStreamingSessionDescriptionWrapper& Offer);

	/**
	 * Send an answer created from a Peer Connection to the signalling server.
	 * @param Answer The answer object created from calling CreateAnswer on a Peer Connection.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void SendAnswer(const FPixelStreamingSessionDescriptionWrapper& Answer);

	/**
	 * Send an Ice Candidate to the signalling server that is generated from a Peer Connection.
	 * @param Candidate The Ice Candidate object generated from a Peer Connection.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming")
	void SendIceCandidate(const FPixelStreamingIceCandidateWrapper& CandidateWrapper);

	/**
	 * Fired when the signalling connection is successfully established.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentConnected OnConnected;

	/**
	 * Fired if the connection failed or an error occurs during the connection. If this is fired at any point the connection should be considered closed.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentConnectionError OnConnectionError;

	/**
	 * Fired when the connection successfully closes.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentDisconnected OnDisconnected;

	/**
	 * Fired when the connection receives a config message from the server. This is the earliest place where the peer connection can be initialized.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentConfig OnConfig;

	/**
	 * Fired when the connection receives an offer from the server. This means there is media being offered up to this connection. Forward to the peer connection.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentOffer OnOffer;

	/**
	 * Fired when the connection receives an answer from the server. The streamer is answering a previously sent offer by us. Forward to the peer connection.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentAnswer OnAnswer;

	/**
	 * Fired when the server sends through an ice candidate. Forward this information on to the peer connection.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingSignallingComponentIceCandidate OnIceCandidate;

	/**
	 * If this media source is set we will use its supplied URL instead of the Url parameter on the connect call.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Properties", META = (DisplayName = "Stream Media Source", AllowPrivateAccess = true))
	TObjectPtr<UStreamMediaSource> MediaSource;

	// Used by the async bp node
	void AsyncRequestStreamerList(const TFunction<void(const TArray<FString>&)>& Callback);

protected:
	class FSignallingObserver : public IPixelStreamingSignallingConnectionObserver
	{
	public:
		FSignallingObserver(UPixelStreamingSignallingComponent* InParent)
			: Parent(InParent) {}

		//
		// ISignallingServerConnectionObserver implementation.
		//
		virtual void OnSignallingConnected() override;
		virtual void OnSignallingDisconnected(int32 StatusCode, const FString& Reason, bool bWasClean) override;
		virtual void OnSignallingError(const FString& ErrorMsg) override;
		virtual void OnSignallingConfig(const webrtc::PeerConnectionInterface::RTCConfiguration& Config) override;
		virtual void OnSignallingSessionDescription(webrtc::SdpType Type, const FString& Sdp) override;
		virtual void OnSignallingRemoteIceCandidate(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) override;
		virtual void OnSignallingPeerDataChannels(int32 SendStreamId, int32 RecvStreamId) override;
		virtual void OnSignallingPlayerCount(uint32 Count) override;
		virtual void OnSignallingPlayerConnected(FPixelStreamingPlayerId PlayerId, const FPixelStreamingPlayerConfig& PlayerConfig, bool bSendOffer) override;
		virtual void OnSignallingPlayerDisconnected(FPixelStreamingPlayerId PlayerId) override;
		virtual void OnSignallingSFUPeerDataChannels(FPixelStreamingPlayerId SFUId, FPixelStreamingPlayerId PlayerId, int32 SendStreamId, int32 RecvStreamId) override;
		virtual void OnSignallingSessionDescription(FPixelStreamingPlayerId PlayerId, webrtc::SdpType Type, const FString& Sdp) override {}
		virtual void OnSignallingRemoteIceCandidate(FPixelStreamingPlayerId PlayerId, const FString& SdpMid, int SdpMLineIndex, const FString& Sdp) override {}
		virtual void OnSignallingStreamerList(const TArray<FString>& StreamerList) override;
		virtual void OnPlayerRequestsBitrate(FPixelStreamingPlayerId PlayerId, int MinBitrate, int MaxBitrate) override;

	private:
		TObjectPtr<UPixelStreamingSignallingComponent> Parent;
	};

	TFunction<void(const TArray<FString>&)> OnStreamerListCallback;

private:
	TUniquePtr<IPixelStreamingSignallingConnection> SignallingConnection;
	TSharedPtr<IPixelStreamingSignallingConnectionObserver> SignallingServerConnectionObserver;
};
