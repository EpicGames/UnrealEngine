// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreamingWebRTCIncludes.h"
#include "PixelStreamingWebRTCWrappers.generated.h"

/**
 * A blueprint wrapper for an RTCConfiguration from webrtc so it can be passed around in blueprints.
 */
USTRUCT(BlueprintType, Category = "PixelStreaming")
struct PIXELSTREAMINGPLAYER_API FPixelStreamingRTCConfigWrapper
{
	GENERATED_USTRUCT_BODY();
	webrtc::PeerConnectionInterface::RTCConfiguration Config;
};

/**
 * A blueprint wrapper for an SessionDescriptionInterface from webrtc so it can be passed around in blueprints.
 */
USTRUCT(BlueprintType, Category = "PixelStreaming")
struct PIXELSTREAMINGPLAYER_API FPixelStreamingSessionDescriptionWrapper
{
	GENERATED_USTRUCT_BODY()
	const webrtc::SessionDescriptionInterface* SDP = nullptr;
};

/**
 * A blueprint wrapper for an IceCandidateInterface from webrtc so it can be passed around in blueprints.
 */
USTRUCT(BlueprintType, Category = "PixelStreaming")
struct PIXELSTREAMINGPLAYER_API FPixelStreamingIceCandidateWrapper
{
	GENERATED_USTRUCT_BODY()

	FPixelStreamingIceCandidateWrapper() = default;

	explicit FPixelStreamingIceCandidateWrapper(const webrtc::IceCandidateInterface& Candidate)
	{
		Mid = UTF8_TO_TCHAR(Candidate.sdp_mid().c_str());
		MLineIndex = Candidate.sdp_mline_index();
		std::string TempStr;
		verifyf(Candidate.ToString(&TempStr), TEXT("Failed to serialize IceCandidate"));
		SDP = UTF8_TO_TCHAR(TempStr.c_str());
	}

	FPixelStreamingIceCandidateWrapper(const FString& SdpMid, int SdpMLineIndex, const FString& Sdp)
	{
		Mid = TCHAR_TO_UTF8(*SdpMid);
		MLineIndex = SdpMLineIndex;
		SDP = TCHAR_TO_UTF8(*Sdp);
	}

	TUniquePtr<webrtc::IceCandidateInterface> ToWebRTC() const
	{
		webrtc::SdpParseError Error;
		TUniquePtr<webrtc::IceCandidateInterface> Candidate = TUniquePtr<webrtc::IceCandidateInterface>(webrtc::CreateIceCandidate(TCHAR_TO_UTF8(*Mid), MLineIndex, TCHAR_TO_UTF8(*SDP), &Error));
		verifyf(Candidate, TEXT("Failed to create ICE candicate: %hs"), Error.description.c_str());
		return Candidate;
	}

	FString Mid;
	int MLineIndex = -1;
	FString SDP;
};
