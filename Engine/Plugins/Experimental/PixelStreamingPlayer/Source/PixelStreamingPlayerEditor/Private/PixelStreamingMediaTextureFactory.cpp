// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingMediaTextureFactory.h"

#include <AssetTypeCategories.h>
#include "PixelStreamingMediaTexture.h"

#define LOCTEXT_NAMESPACE "PixelStreaming"

UPixelStreamingMediaTextureFactory::UPixelStreamingMediaTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;

	SupportedClass = UPixelStreamingMediaTexture::StaticClass();
}

FText UPixelStreamingMediaTextureFactory::GetDisplayName() const
{
	return LOCTEXT("MediaTextureFactoryDisplayName", "Pixel Streaming Media Texture");
}

uint32 UPixelStreamingMediaTextureFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Textures;
}

UObject* UPixelStreamingMediaTextureFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreamingMediaTexture* Resource = NewObject<UPixelStreamingMediaTexture>(InParent, InName, Flags | RF_Transactional))
	{
		Resource->UpdateResource();
		return Resource;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
