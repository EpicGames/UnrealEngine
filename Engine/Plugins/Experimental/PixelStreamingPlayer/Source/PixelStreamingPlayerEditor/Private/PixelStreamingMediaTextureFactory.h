// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Factories/Factory.h>
#include <UObject/Object.h>

#include "PixelStreamingMediaTextureFactory.generated.h"

UCLASS()
class UPixelStreamingMediaTextureFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	public:
		virtual FText GetDisplayName() const override;
		virtual uint32 GetMenuCategories() const override;
		virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
